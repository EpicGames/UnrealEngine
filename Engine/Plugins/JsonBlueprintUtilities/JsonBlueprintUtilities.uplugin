{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Json Blueprint Utilities",
	"Description": "Json functionality for Blueprint.",
	"Category": "Blueprints",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "JsonBlueprintUtilities",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "JsonBlueprintGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	]
}
