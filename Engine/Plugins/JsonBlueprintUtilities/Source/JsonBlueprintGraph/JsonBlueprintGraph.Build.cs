// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class JsonBlueprintGraph : ModuleRules
{
	public JsonBlueprintGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetRegistry",
				"BlueprintGraph",
				"Core",
				"CoreUObject",
				"Engine",
				"GraphEditor",
				"InputCore",
				"Json",
				"JsonBlueprintUtilities",
				"Kismet",
				"KismetCompiler",
				"KismetWidgets",
				"Slate",
				"SlateCore",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",		
					"Projects",
					"PropertyEditor",
					"UnrealEd",
				}
			);
		}
	}
}
