// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetJsonField.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "JsonBlueprintFunctionLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "K2Node_GetJsonField"

UK2Node_GetJsonField::UK2Node_GetJsonField()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UJsonBlueprintFunctionLibrary, GetField), UJsonBlueprintFunctionLibrary::StaticClass());
}

void UK2Node_GetJsonField::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	/* Structure to hold one-time initialization */
	struct FPinStatics
	{
		int32 InputJsonObjectPinIndex = -1;
		int32 InputFieldNamePinIndex = -1;
		int32 InputArrayIndexPinIndex = -1;

		int32 OutputValuePinIndex = -1;
		int32 OutputResultPinIndex = -1;

		FPinStatics(const TArray<UEdGraphPin*>& InPins)
		{
			// Pins as per UJsonBlueprintFunctionLibrary::GetField(...)
			InputJsonObjectPinIndex = FindPinByName(InPins, TEXT("self"));		// default name, won't change
			InputFieldNamePinIndex = FindPinByName(InPins, TEXT("FieldName"));

			OutputValuePinIndex = FindPinByName(InPins, TEXT("OutValue"));
			OutputResultPinIndex = FindPinByName(InPins, TEXT("ReturnValue"));	// default name, won't change
		}

		static int32 FindPinByName(const TArray<UEdGraphPin*>& InPins, const FName& InName)
		{
			return InPins.IndexOfByPredicate([&InName](const UEdGraphPin* InPin)
			{
				return InPin->GetFName() == InName;				
			});			
		}
	};
	static FPinStatics PinInfo(Pins);

	UEdGraphPin* InputJsonObjectPin = GetPinAt(PinInfo.InputJsonObjectPinIndex);
	check(InputJsonObjectPin);
	
	UEdGraphPin* InputFieldNamePin = GetPinAt(PinInfo.InputFieldNamePinIndex);	
	check(InputFieldNamePin);

	UEdGraphPin* OutputValuePin = GetPinAt(PinInfo.OutputValuePinIndex);
	check(OutputValuePin);
	
	UEdGraphPin* OutputResultPin = GetPinAt(PinInfo.OutputResultPinIndex);
	check(OutputResultPin);
}

FText UK2Node_GetJsonField::GetTooltipText() const
{
	return LOCTEXT("TooltipText", "Get the specified Json Field, the type is determined by the output target property.");
}

bool UK2Node_GetJsonField::IsNodePure() const
{
	return false;
}

void UK2Node_GetJsonField::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	const UClass* ActionKey = GetClass();
	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GetJsonField::GetMenuCategory() const
{
	static FText MenuCategory = LOCTEXT("MenuCategory", "Json");
	return MenuCategory;
}

void UK2Node_GetJsonField::NotifyInputChanged() const
{
	if (UBlueprint* BP = GetBlueprint())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
	}

	UEdGraph* Graph = GetGraph();
	Graph->NotifyNodeChanged(this);
}

#undef LOCTEXT_NAMESPACE
