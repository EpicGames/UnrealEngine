// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SetJsonField.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "JsonBlueprintFunctionLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "K2Node_SetJsonField"

UK2Node_SetJsonField::UK2Node_SetJsonField()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UJsonBlueprintFunctionLibrary, SetField), UJsonBlueprintFunctionLibrary::StaticClass());
}

void UK2Node_SetJsonField::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	/* Structure to hold one-time initialization */
	struct FPinStatics
	{
		int32 InputJsonObjectPinIndex = -1;
		int32 InputFieldNamePinIndex = -1;
		int32 InputValuePinIndex = -1;

		int32 OutputResultPinIndex = -1;

		FPinStatics(const TArray<UEdGraphPin*>& InPins)
		{
			// Pins as per UJsonBlueprintFunctionLibrary::SetField(...)
			InputJsonObjectPinIndex = FindPinByName(InPins, TEXT("self"));		// default name, won't change
			InputFieldNamePinIndex = FindPinByName(InPins, TEXT("FieldName"));
			InputValuePinIndex = FindPinByName(InPins, TEXT("Value"));

			OutputResultPinIndex = FindPinByName(InPins, TEXT("ReturnValue"));	// default name, won't change
		}

		static int32 FindPinByName(const TArray<UEdGraphPin*>& InPins, const FName& InName)
		{
			return InPins.IndexOfByPredicate([&InName](const UEdGraphPin* InPin)
			{
				return InPin->GetFName() == InName;				
			});			
		}
	};
	static FPinStatics PinInfo(Pins);

	UEdGraphPin* InputJsonObjectPin = GetPinAt(PinInfo.InputJsonObjectPinIndex);
	check(InputJsonObjectPin);
	
	UEdGraphPin* InputFieldNamePin = GetPinAt(PinInfo.InputFieldNamePinIndex);	
	check(InputFieldNamePin);

	UEdGraphPin* InputValuePin = GetPinAt(PinInfo.InputValuePinIndex);	
	check(InputValuePin);

	UEdGraphPin* OutputResultPin = GetPinAt(PinInfo.OutputResultPinIndex);
	check(OutputResultPin);
}

FText UK2Node_SetJsonField::GetTooltipText() const
{
	return LOCTEXT("TooltipText", "Set the specified Json Field, the type is determined by the input target property.");
}

bool UK2Node_SetJsonField::IsNodePure() const
{
	return false;
}

void UK2Node_SetJsonField::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	const UClass* ActionKey = GetClass();
	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_SetJsonField::GetMenuCategory() const
{
	static FText MenuCategory = LOCTEXT("MenuCategory", "Json");
	return MenuCategory;
}

void UK2Node_SetJsonField::NotifyInputChanged() const
{
	if (UBlueprint* BP = GetBlueprint())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
	}

	UEdGraph* Graph = GetGraph();
	Graph->NotifyNodeChanged(this);
}

#undef LOCTEXT_NAMESPACE
