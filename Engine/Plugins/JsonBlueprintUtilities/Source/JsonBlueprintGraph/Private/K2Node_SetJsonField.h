// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_CallFunction.h"

#include "K2Node_SetJsonField.generated.h"

/** A custom node that sets a JsonObject field. */
UCLASS()
class JSONBLUEPRINTGRAPH_API UK2Node_SetJsonField
	: public UK2Node_CallFunction
{
	GENERATED_BODY()

public:
	UK2Node_SetJsonField();

	// //~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;	
	// //~ End UEdGraphNode Interface.

	//~ Begin K2Node Interface
	virtual bool IsNodePure() const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End K2Node Interface

protected:
	void NotifyInputChanged() const;
};
