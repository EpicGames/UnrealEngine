// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_StructToJsonString.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "JsonBlueprintFunctionLibrary.h"

#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "K2Node_StructToJsonString"

UK2Node_StructToJsonString::UK2Node_StructToJsonString()
{
	FunctionReference.SetExternalMember(
		GET_FUNCTION_NAME_CHECKED(UJsonBlueprintFunctionLibrary, StructToJsonString),
		UJsonBlueprintFunctionLibrary::StaticClass());
}

void UK2Node_StructToJsonString::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	struct FPinStatics
	{
		int32 InputStructPinIndex = -1;
		int32 OutputStringIndex = -1;

		FPinStatics(const TArray<UEdGraphPin*>& InPins)
		{
			InputStructPinIndex = FindPinByName(InPins, TEXT("Struct"));
			OutputStringIndex = FindPinByName(InPins, TEXT("OutJsonString"));
		}

		static int32 FindPinByName(const TArray<UEdGraphPin*>& InPins, const FName& InName)
		{
			return InPins.IndexOfByPredicate([&InName](const UEdGraphPin* InPin)->bool
			{
				return InPin->GetFName() == InName;
			});
		}
		
	} static PinInfo{Pins};

	UEdGraphPin* InputStructPin = GetPinAt(PinInfo.InputStructPinIndex);
	check(InputStructPin);

	UEdGraphPin* OutputStringPin = GetPinAt(PinInfo.OutputStringIndex);
	check(OutputStringPin);
	
}

FText UK2Node_StructToJsonString::GetTooltipText() const
{
	return LOCTEXT("TooltipText", "Converts a Structure to a raw JSON string.");
}

bool UK2Node_StructToJsonString::IsNodePure() const
{
	return false;
}

void UK2Node_StructToJsonString::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	const UClass* ActionKey = GetClass();
	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_StructToJsonString::GetMenuCategory() const
{
	static FText MenuCategory = LOCTEXT("MenuCategory", "Json");
	return MenuCategory;
}

void UK2Node_StructToJsonString::NotifyInputChanged() const
{
	if (UBlueprint* BP = GetBlueprint())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
	}

	UEdGraph* Graph = GetGraph();
	Graph->NotifyNodeChanged(this);
}

#undef LOCTEXT_NAMESPACE
