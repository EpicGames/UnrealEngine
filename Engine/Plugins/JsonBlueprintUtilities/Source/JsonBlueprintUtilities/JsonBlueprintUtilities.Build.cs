// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class JsonBlueprintUtilities : ModuleRules
{
	public JsonBlueprintUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
                "CoreUObject",
                "Engine",
            });

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Json",
				"JsonUtilities"
			});
	}
}
