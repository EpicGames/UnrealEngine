// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonBlueprintFunctionLibrary.h"

#include "Blueprint/BlueprintExceptionInfo.h"
#include "JsonObjectConverter.h"
#include "JsonObjectWrapper.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "JsonBlueprintFunctionLibrary"

bool UJsonBlueprintFunctionLibrary::FromString(
	const FString& JsonString,
	FJsonObjectWrapper& OutJsonObject)
{
	return OutJsonObject.JsonObjectFromString(JsonString);
}

bool UJsonBlueprintFunctionLibrary::FromFile(
	const FFilePath& File,
	FJsonObjectWrapper& OutJsonObject)
{
	if(!FPaths::FileExists(File.FilePath))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("File not found: %s"), *File.FilePath), ELogVerbosity::Error);
		return false;
	}
	
	FString JsonString;
	if(!FFileHelper::LoadFileToString(JsonString, *File.FilePath))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Error loading file: %s"), *File.FilePath), ELogVerbosity::Error);
		return false;
	}
	
	return FromString(JsonString, OutJsonObject);
}

bool UJsonBlueprintFunctionLibrary::ToString(const FJsonObjectWrapper& JsonObject, FString& OutJsonString)
{
	if (!JsonObject.JsonObjectToString(OutJsonString))
	{
		FFrame::KismetExecutionMessage(TEXT("Failed to convert JSON Object to string"), ELogVerbosity::Error);
		return false;
	}

	return true;
}

bool UJsonBlueprintFunctionLibrary::ToFile(const FJsonObjectWrapper& JsonObject, const FFilePath& File)
{
	if (!FPaths::DirectoryExists(FPaths::GetPath(File.FilePath)))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Directory not found: %s"), *FPaths::GetPath(File.FilePath)), ELogVerbosity::Error);
		return false;
	}

	if (File.FilePath.IsEmpty())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("FileName cannot be empty")), ELogVerbosity::Error);
		return false;
	}

	if (FString JsonString; ToString(JsonObject, JsonString))
	{
		const bool& bResult = FFileHelper::SaveStringToFile(JsonString, *File.FilePath);
		if (!bResult)
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Failed to save JSON to file. %s"), *JsonString), ELogVerbosity::Error);
		}

		return bResult;
	}

	return false;
}

DEFINE_FUNCTION(UJsonBlueprintFunctionLibrary::execGetField)
{
	P_GET_STRUCT_REF(FJsonObjectWrapper, JsonObject);
	P_GET_PROPERTY(FStrProperty, FieldName);

	Stack.StepCompiledIn<FProperty>(nullptr);
	FProperty* ValueProp = Stack.MostRecentProperty;
	void* ValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (!ValueProp || !ValuePtr)
	{
		const FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("GetField_MissingOutputProperty", "Failed to resolve the output parameter for GetField.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult;
	if (FieldName.IsEmpty())
	{
		FStructProperty* StructProperty = static_cast<FStructProperty*>(ValueProp);
		if (!StructProperty)
		{
			bResult = false;
			*StaticCast<bool*>(RESULT_PARAM) = bResult;
			return;
		}
		
		P_NATIVE_BEGIN
		bResult = FJsonObjectConverter::JsonObjectToUStruct(JsonObject.JsonObject.ToSharedRef(), StructProperty->Struct, ValuePtr);
		P_NATIVE_END
		*StaticCast<bool*>(RESULT_PARAM) = bResult;
		return;
	}

	P_NATIVE_BEGIN
	bResult = JsonFieldToProperty(FieldName, JsonObject, ValueProp, ValuePtr);
	P_NATIVE_END

	*static_cast<bool*>(RESULT_PARAM) = bResult;
}

DEFINE_FUNCTION(UJsonBlueprintFunctionLibrary::execSetField)
{
	P_GET_STRUCT_REF(FJsonObjectWrapper, JsonObject);
	P_GET_PROPERTY(FStrProperty, FieldName);

	Stack.StepCompiledIn<FProperty>(nullptr);
	FProperty* SourceProperty = Stack.MostRecentProperty;
	void* SourceValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (!SourceProperty || !SourceValuePtr)
	{
		const FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("SetField_MissingInputProperty", "Failed to resolve the input parameter for SetField.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult;
	if (FieldName.IsEmpty())
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(SourceProperty);
		if (!StructProperty)
		{
			bResult = false;
			*StaticCast<bool*>(RESULT_PARAM) = bResult;
			return;
		}

		P_NATIVE_BEGIN
		bResult = FJsonObjectConverter::UStructToJsonObject(StructProperty->Struct, SourceValuePtr, JsonObject.JsonObject.ToSharedRef());
		P_NATIVE_END
	}
	else
	{
		P_NATIVE_BEGIN
		bResult = PropertyToJsonField(FieldName, SourceProperty, SourceValuePtr, JsonObject);
		P_NATIVE_END
	}

	// If successful, refresh the stored JsonString
	P_NATIVE_BEGIN
	if (bResult)
	{
		if (!FJsonSerializer::Serialize(JsonObject.JsonObject.ToSharedRef(), TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonObject.JsonString)))
		{
			FFrame::KismetExecutionMessage(TEXT("Error serializing JSON Object."), ELogVerbosity::Error);
			bResult = false;
		}
	}
	P_NATIVE_END
	
	*StaticCast<bool*>(RESULT_PARAM) = bResult;
}

DEFINE_FUNCTION(UJsonBlueprintFunctionLibrary::execStructToJsonString)
{
	Stack.StepCompiledIn<FProperty>(nullptr);
	FProperty* ValueProperty = Stack.MostRecentProperty;
	void* ValuePtr = Stack.MostRecentPropertyAddress;

	PARAM_PASSED_BY_REF(OutJsonString, FStrProperty, FString);

	P_FINISH;

	if (!ValueProperty || !ValuePtr)
	{
		const FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("StructToJsonString_MissingInputProperty", "Failed to resolve the input parameter for StructToJsonString.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult;
	FStructProperty* const StructProperty = CastField<FStructProperty>(ValueProperty);
	if (!StructProperty)
	{
		bResult = false;
		*StaticCast<bool*>(RESULT_PARAM) = bResult;
		return;
	}
	
	P_NATIVE_BEGIN
	bResult = FJsonObjectConverter::UStructToJsonObjectString(StructProperty->Struct, ValuePtr, OutJsonString);
	P_NATIVE_END

	*StaticCast<bool*>(RESULT_PARAM) = bResult;
}

bool UJsonBlueprintFunctionLibrary::HasField(const FJsonObjectWrapper& JsonObject, const FString& FieldName)
{
	return JsonObject.JsonObject->HasField(FieldName);
}

bool UJsonBlueprintFunctionLibrary::GetFieldNames(const FJsonObjectWrapper& JsonObject, TArray<FString>& FieldNames)
{
	FieldNames.Reserve(JsonObject.JsonObject->Values.Num());
	for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Field : JsonObject.JsonObject->Values)
	{
		FieldNames.Add(FString(Field.Key));
	}

	return true;
}

bool UJsonBlueprintFunctionLibrary::JsonFieldToProperty(
	const FString& FieldName,
	const FJsonObjectWrapper& SourceObject,
	FProperty* TargetProperty,
	void* TargetValuePtr)
{
	check(SourceObject.JsonObject.IsValid());
	check(TargetProperty && TargetValuePtr);

	// Check that field with name exists
	const TSharedPtr<FJsonValue> JsonValue = SourceObject.JsonObject->TryGetField(FieldName);
	if(!JsonValue.IsValid())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Field '%s' was not found on the provided JSON object."), *FieldName), ELogVerbosity::Warning);
		return false;
	}
	
	return FJsonObjectConverter::JsonValueToUProperty(JsonValue, TargetProperty, TargetValuePtr);
}

bool UJsonBlueprintFunctionLibrary::PropertyToJsonField(
	const FString& FieldName,
	FProperty* SourceProperty,
	const void* SourceValuePtr,
	FJsonObjectWrapper& TargetObject)
{
	check(SourceProperty && SourceValuePtr);

	if(!TargetObject.JsonObject.IsValid())
	{
		TargetObject.JsonObject = MakeShared<FJsonObject>();
	}

	TargetObject.JsonObject->SetField(FieldName, FJsonObjectConverter::UPropertyToJsonValue(SourceProperty, SourceValuePtr));
	return true;
}

#undef LOCTEXT_NAMESPACE
