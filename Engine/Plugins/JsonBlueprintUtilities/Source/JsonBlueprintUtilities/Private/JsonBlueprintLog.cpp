// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonBlueprintLog.h"

DEFINE_LOG_CATEGORY(LogJsonBlueprintUtilities);
