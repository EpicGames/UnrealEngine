// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "JsonObjectWrapper.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "JsonBlueprintFunctionLibrary.generated.h"

/** */
UCLASS(BlueprintType)
class JSONBLUEPRINTUTILITIES_API UJsonBlueprintFunctionLibrary final : public UBlueprintFunctionLibrary
{ 
	GENERATED_BODY()

public:
	/** Creates a JsonObject from the provided Json string. */
	UFUNCTION(BlueprintCallable, Category="Json", meta = (DisplayName="Load Json from String"))
	static UPARAM(DisplayName="Success") bool FromString(const FString& JsonString, UPARAM(DisplayName="JsonObject") FJsonObjectWrapper& OutJsonObject);

	/** Creates a JsonObject from the provided Json file. */
	UFUNCTION(BlueprintCallable, Category="Json", meta = (DisplayName="Load Json from File"))
	static UPARAM(DisplayName="Success") bool FromFile(const FFilePath& File, UPARAM(DisplayName="JsonObject") FJsonObjectWrapper& OutJsonObject);

	/** Creates a Json string from the provided JsonObject. */
	UFUNCTION(BlueprintCallable, Category="Json", meta = (DisplayName = "Get Json String"))
	static UPARAM(DisplayName="Success") bool ToString(const FJsonObjectWrapper& JsonObject, UPARAM(DisplayName="String") FString& OutJsonString);
	
	/** Creates a file from the provided JsonObject. */
	UFUNCTION(BlueprintCallable, Category="Json", meta = (DisplayName = "Save Json to File"))
	static UPARAM(DisplayName="Success") bool ToFile(const FJsonObjectWrapper& JsonObject, const FFilePath& File);
	
	/** Gets the value of the specified field. */
	UFUNCTION(BlueprintCallable, CustomThunk, BlueprintInternalUseOnly, Category="Json", meta = (CustomStructureParam = "OutValue", AutoCreateRefTerm = "OutValue"))
	static UPARAM(DisplayName="Success") bool GetField(const FJsonObjectWrapper& JsonObject, const FString& FieldName, UPARAM(DisplayName="Value") int32& OutValue);
	DECLARE_FUNCTION(execGetField);

	/** Adds (new) or sets (existing) the value of the specified field. */
	UFUNCTION(BlueprintCallable, CustomThunk, BlueprintInternalUseOnly, Category="Json", meta = (CustomStructureParam = "Value", AutoCreateRefTerm = "Value"))
	static UPARAM(DisplayName="Success") bool SetField(const FJsonObjectWrapper& JsonObject, const FString& FieldName, const int32& Value);
	DECLARE_FUNCTION(execSetField);

	UFUNCTION(BlueprintCallable, CustomThunk, BlueprintInternalUseOnly, Category="Json", meta = (DisplayName = "Convert Struct To Json String", CustomStructureParam = "Struct", AutoCreateRefTerm = "Struct"))
	static UPARAM(DisplayName="Success") bool StructToJsonString(const int32& Struct, FString& OutJsonString);
	DECLARE_FUNCTION(execStructToJsonString);
	
	/** Checks if the field exists on the object. */
	UFUNCTION(BlueprintCallable, Category="Json")
	static UPARAM(DisplayName="Success") bool HasField(const FJsonObjectWrapper& JsonObject, const FString& FieldName);

	/** Gets all field names on the JsonObject */
	UFUNCTION(BlueprintCallable, Category="Json")
	static UPARAM(DisplayName="Success") bool GetFieldNames(const FJsonObjectWrapper& JsonObject, TArray<FString>& FieldNames);
	
private:
	/** FieldName only used for logging, SrcValue should be the resolved field. */
	static bool JsonFieldToProperty(const FString& FieldName, const FJsonObjectWrapper& SourceObject, FProperty* TargetProperty, void* TargetValuePtr);

	// If FieldName empty, Object created as root, or created with name "Value" otherwise. 
	static bool PropertyToJsonField(const FString& FieldName, FProperty* SourceProperty, const void* SourceValuePtr, FJsonObjectWrapper& TargetObject);
};
